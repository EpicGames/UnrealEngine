// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterLightCardExtenderModule.h"

#if WITH_EDITOR
#include "ISequencerModule.h"
#endif

void FDisplayClusterLightCardExtenderModule::StartupModule()
{
#if WITH_EDITOR
	ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
	SequencerCreatedHandle = SequencerModule.RegisterOnSequencerCreated(FOnSequencerCreated::FDelegate::CreateRaw(this, &FDisplayClusterLightCardExtenderModule::OnSequencerCreated));
#endif
}

void FDisplayClusterLightCardExtenderModule::ShutdownModule()
{
#if WITH_EDITOR
	for (const TWeakPtr<ISequencer, ESPMode::ThreadSafe>& Sequencer : OpenSequencers)
	{
		if (Sequencer.IsValid())
		{
			Sequencer.Pin()->OnGlobalTimeChanged().RemoveAll(this);
			Sequencer.Pin()->OnCloseEvent().RemoveAll(this);
		}
	}

	OpenSequencers.Empty();
	
	if (FModuleManager::Get().IsModuleLoaded("Sequencer"))
	{
		ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>(TEXT("Sequencer"));
		SequencerModule.UnregisterOnSequencerCreated(SequencerCreatedHandle);
	}
#endif
}

#if WITH_EDITOR
void FDisplayClusterLightCardExtenderModule::OnSequencerCreated(TSharedRef<ISequencer> InSequencer)
{
	TWeakPtr<ISequencer> SequencerWeakPtr = InSequencer;
	OpenSequencers.Add(SequencerWeakPtr);
	
	InSequencer->OnGlobalTimeChanged().AddRaw(this, &FDisplayClusterLightCardExtenderModule::OnSequencerTimeChanged, SequencerWeakPtr);
	InSequencer->OnCloseEvent().AddRaw(this, &FDisplayClusterLightCardExtenderModule::OnSequencerClosed);
}

void FDisplayClusterLightCardExtenderModule::OnSequencerTimeChanged(TWeakPtr<ISequencer> InSequencer)
{
	OnSequencerTimeChangedDelegate.Broadcast(InSequencer);
}

void FDisplayClusterLightCardExtenderModule::OnSequencerClosed(TSharedRef<ISequencer> InSequencer)
{
	InSequencer->OnGlobalTimeChanged().RemoveAll(this);
	InSequencer->OnCloseEvent().RemoveAll(this);
	OpenSequencers.Remove(InSequencer);

	// Positions could have reset to their original values.
	OnSequencerTimeChangedDelegate.Broadcast(InSequencer);
}
#endif

IMPLEMENT_MODULE(FDisplayClusterLightCardExtenderModule, DisplayClusterLightCardExtender)
