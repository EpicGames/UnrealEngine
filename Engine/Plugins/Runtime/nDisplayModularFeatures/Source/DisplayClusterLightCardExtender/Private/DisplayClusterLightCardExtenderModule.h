// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDisplayClusterLightCardExtenderModule.h"

class ISequencer;

class FDisplayClusterLightCardExtenderModule : public IDisplayClusterLightCardExtenderModule
{
public:
	
	/** IModuleInterface implementation start */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	/** IModuleInterface implementation end */

#if WITH_EDITOR
	/** Broadcast when any sequencer has its time changed */
	virtual FOnSequencerTimeChanged& GetOnSequencerTimeChanged() override { return OnSequencerTimeChangedDelegate; }

private:
	/** When a sequencer is opened */
	void OnSequencerCreated(TSharedRef<ISequencer> InSequencer);
	
	/** Called when the global time is changed in sequencer */
	void OnSequencerTimeChanged(TWeakPtr<ISequencer> InSequencer);
	
	/** When a sequencer is closed */
	void OnSequencerClosed(TSharedRef<ISequencer> InSequencer);
	
private:
	TArray<TWeakPtr<ISequencer>> OpenSequencers;
	FDelegateHandle SequencerCreatedHandle;

	FOnSequencerTimeChanged OnSequencerTimeChangedDelegate;
#endif
};
