// Copyright Epic Games, Inc. All Rights Reserved.

#include "IDisplayClusterLightCardActorExtender.h"

const FName IDisplayClusterLightCardActorExtender::ModularFeatureName = "DisplayClusterLightCardActorExtender";
