// Copyright Epic Games, Inc. All Rights Reserved.

#include "StageActor/DisplayClusterWeakStageActorPtr.h"
#include "StageActor/IDisplayClusterStageActor.h"

#include "GameFramework/Actor.h"

FDisplayClusterWeakStageActorPtr::FDisplayClusterWeakStageActorPtr(const AActor* InActorPtr)
{
	if (InActorPtr)
	{
		if (ensureMsgf(InActorPtr->Implements<UDisplayClusterStageActor>(),
			TEXT("Actor passed to FDisplayClusterWeakStageActorPtr must implement or inherit IDisplayClusterStageActor interface")))
		{
			ObjectPtr = InActorPtr;
		}
		else
		{
			ObjectPtr = nullptr;
		}
	}
}

IDisplayClusterStageActor* FDisplayClusterWeakStageActorPtr::Get() const
{
	return Cast<IDisplayClusterStageActor>(ObjectPtr.Get());
}

AActor* FDisplayClusterWeakStageActorPtr::AsActor() const
{
	return Cast<AActor>(ObjectPtr.Get());
}

AActor* FDisplayClusterWeakStageActorPtr::AsActorChecked() const
{
	return CastChecked<AActor>(ObjectPtr.Get());
}
