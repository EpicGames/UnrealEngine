// Copyright Epic Games, Inc. All Rights Reserved.

#include "StageActor/IDisplayClusterStageActor.h"

#include "GameFramework/Actor.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

const FRotator IDisplayClusterStageActor::PlaneMeshRotation = FRotator(0.0f, -90.0f, 90.0f);

struct FStageActorHelper
{
	static AActor* GetActor(IDisplayClusterStageActor* InInterface)
	{
		AActor* Actor = Cast<AActor>(InInterface);
		ensureMsgf(Actor != nullptr, TEXT("IDisplayClusterStageActor should be implemented on Actor types only"));
		return Actor;
	}

	static const AActor* GetActorConst(const IDisplayClusterStageActor* InInterface)
	{
		const AActor* Actor = Cast<AActor>(InInterface);
		ensureMsgf(Actor != nullptr, TEXT("IDisplayClusterStageActor should be implemented on Actor types only"));
		return Actor;
	}
};

const TSet<FName>& IDisplayClusterStageActor::GetPositionalPropertyNames() const
{
	static TSet<FName> DefaultPositionalPropertyNames =
		{
			GET_MEMBER_NAME_CHECKED(FDisplayClusterPositionalParams, DistanceFromCenter),
			GET_MEMBER_NAME_CHECKED(FDisplayClusterPositionalParams, Longitude),
			GET_MEMBER_NAME_CHECKED(FDisplayClusterPositionalParams, Latitude),
			GET_MEMBER_NAME_CHECKED(FDisplayClusterPositionalParams, Spin),
			GET_MEMBER_NAME_CHECKED(FDisplayClusterPositionalParams, Pitch),
			GET_MEMBER_NAME_CHECKED(FDisplayClusterPositionalParams, Yaw),
			GET_MEMBER_NAME_CHECKED(FDisplayClusterPositionalParams, RadialOffset),
			GET_MEMBER_NAME_CHECKED(FDisplayClusterPositionalParams, Scale),
			TEXT("UVCoordinates") // Not currently a member of positional params
		};
	
	return DefaultPositionalPropertyNames;
}

FName IDisplayClusterStageActor::GetPositionalPropertiesMemberName() const
{
	return NAME_None;
}

FBox IDisplayClusterStageActor::GetBoxBounds(bool bLocalSpace) const
{
	if (const AActor* Actor = FStageActorHelper::GetActorConst(this))
	{
		return Actor->GetComponentsBoundingBox(/* bNonColliding */ true);
	}
	return FBox();
}

bool IDisplayClusterStageActor::IsProxy() const
{
	if (const AActor* Actor = FStageActorHelper::GetActorConst(this))
	{
		return Actor->HasAnyFlags(RF_Transient);
	}

	return false;
}

void IDisplayClusterStageActor::UpdateStageActorTransform()
{
	const FTransform Transform = PositionalParamsToActorTransform(GetPositionalParams(), GetOrigin());
	if (AActor* Actor = FStageActorHelper::GetActor(this))
	{
#if WITH_EDITOR
		const bool bTransformChanged = !Actor->GetActorTransform().Equals(Transform, 0.f);
#endif
		
		Actor->SetActorTransform(Transform);

#if WITH_EDITOR
		if (bTransformChanged)
		{
			if (FProperty* TransformProperty = Actor->GetRootComponent() ?
				Actor->GetRootComponent()->GetClass()->FindPropertyByName(USceneComponent::GetRelativeLocationPropertyName())
				: nullptr)
			{
				FPropertyChangedEvent PropertyChangedEvent(TransformProperty, EPropertyChangeType::ValueSet);
				FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(Actor, PropertyChangedEvent);
			}
		}
#endif

#if WITH_EDITOR
		UpdateEditorGizmos();
#endif
	}
}

FTransform IDisplayClusterStageActor::GetStageActorTransform(bool bRemoveOrigin) const
{
	FTransform Transform;

	if (const AActor* Actor = FStageActorHelper::GetActorConst(this))
	{
		FVector Location = Actor->GetActorLocation();
		if (bRemoveOrigin)
		{
			Location -= GetOrigin().GetLocation();
		}
		Transform.SetLocation(Location);
	
		// Use the actor's orientation, but remove the plane mesh rotation so that the returned transform's local x axis
		// points radially inwards to match engine convention
		const FQuat StageActorOrientation = Actor->GetActorQuat() * PlaneMeshRotation.Quaternion().Inverse();

		Transform.SetRotation(StageActorOrientation);
	}
	
	return Transform;
}

void IDisplayClusterStageActor::SetPositionalParams(
	const FDisplayClusterPositionalParams& InParams)
{
	SetPitch(InParams.Pitch);
	SetYaw(InParams.Yaw);
	SetSpin(InParams.Spin);
	SetLatitude(InParams.Latitude);
	SetLongitude(InParams.Longitude);
	SetDistanceFromCenter(InParams.DistanceFromCenter);
	SetScale(InParams.Scale);
	SetRadialOffset(InParams.RadialOffset);
}

FDisplayClusterPositionalParams IDisplayClusterStageActor::GetPositionalParams() const
{
	FDisplayClusterPositionalParams Params = FDisplayClusterPositionalParams();

	Params.Pitch = GetPitch();
	Params.Yaw = GetYaw();
	Params.Spin = GetSpin();
	Params.Latitude = GetLatitude();
	Params.Longitude = GetLongitude();
	Params.DistanceFromCenter = GetDistanceFromCenter();
	Params.Scale = GetScale();
	Params.RadialOffset = GetRadialOffset();
	
	ClampLatitudeAndLongitude(Params.Latitude, Params.Longitude);
	
	return Params;
}

void IDisplayClusterStageActor::UpdatePositionalParamsFromTransform()
{
	if (const AActor* Actor = FStageActorHelper::GetActorConst(this))
	{
		const FDisplayClusterPositionalParams PositionalParams = TransformToPositionalParams(
		Actor->GetTransform(), GetOrigin(), GetRadialOffset());

		SetPositionalParams(PositionalParams);

		if (IsAlwaysFlushToWall())
		{
			// TODO: Implement this once there is a way to determine which root actor owns this stage actor. 
		}
	}
}

void IDisplayClusterStageActor::ClampLatitudeAndLongitude(double& InOutLatitude, double& InOutLongitude)
{
	double Latitude = InOutLatitude;
	double Longitude = InOutLongitude;
	if (Longitude < 0 || Longitude > 360)
	{
		Longitude = FRotator::ClampAxis(Longitude);
	}

	if (Latitude < -90 || Latitude > 90)
	{
		// If latitude exceeds [-90, 90], mod it back into the appropriate range, and apply a shift of 180 degrees if
		// needed to the longitude, to allow the latitude to be continuous (increasing latitude indefinitely should result in the LC 
		// orbiting around a polar great circle)
		const double Parity = FMath::Fmod(FMath::Abs(Latitude) + 90.f, 360.f) - 180.f;
		const double DeltaLongitude = Parity > 1 ? 180.f : 0.f;

		double LatMod = FMath::Fmod(Latitude + 90.f, 180.f);
		if (LatMod < 0.f)
		{
			LatMod += 180.f;
		}

		Latitude = LatMod - 90.f;
		Longitude = FRotator::ClampAxis(Longitude + DeltaLongitude);
	}

	InOutLatitude = Latitude;
	InOutLongitude = Longitude;
}

FTransform IDisplayClusterStageActor::PositionalParamsToActorTransform(const FDisplayClusterPositionalParams& InParams, const FTransform& InOrigin)
{
	const double Inclination = FMath::DegreesToRadians(90.0 - InParams.Latitude);
	const double Azimuth = FMath::DegreesToRadians(InParams.Longitude + 180.0);
	const double Radius = InParams.DistanceFromCenter + InParams.RadialOffset;
	
	FVector Location;
	{
		const double CosInc = FMath::Cos(Inclination);
		const double SinInc = FMath::Sin(Inclination);
		const double CosAz = FMath::Cos(Azimuth);
		const double SinAz = FMath::Sin(Azimuth);
		Location.X = Radius * SinInc * CosAz;
		Location.Y = Radius * SinInc * SinAz;
		Location.Z = Radius * CosInc;
	}
	
	const FRotator SphericalRotation = FRotator(-InParams.Latitude, InParams.Longitude, 0.0);
	const FRotator StageActorOrientation = FRotator(-InParams.Pitch, InParams.Yaw, InParams.Spin);

	// The origin point may be rotated (ie the DCRA may have a rotation applied)
	const FVector RotatedLocation = InOrigin.GetRotation().RotateVector(Location);
	const FQuat InitialRotation = InOrigin.GetRotation();
	
	FTransform Result;
	Result.SetLocation(InOrigin.GetLocation() + RotatedLocation);
	Result.SetRotation((InitialRotation * SphericalRotation.Quaternion() * StageActorOrientation.Quaternion() * PlaneMeshRotation.Quaternion()).Rotator().Quaternion());
	Result.SetScale3D(FVector(InParams.Scale, 1.f));

	return Result;
}

FDisplayClusterPositionalParams IDisplayClusterStageActor::TransformToPositionalParams(
	const FTransform& InTransform, const FTransform& InOrigin, double InRadialOffset)
{
	FDisplayClusterPositionalParams PositionalParams;
	
	const FVector Location = InOrigin.GetRotation().UnrotateVector(InTransform.GetLocation() - InOrigin.GetLocation());
	
	const double Radius = FVector::Dist(InTransform.GetLocation(), InOrigin.GetLocation());
	const double DistanceFromCenter = Radius - InRadialOffset;
	
	double Latitude = 90.0 - FMath::RadiansToDegrees(FMath::Acos(Location.Z / Radius));
	double Longitude = FMath::RadiansToDegrees(FMath::Atan2(Location.Y, Location.X)) - 180.0;

	// Clamp is necessary to maintain correct rotation
	ClampLatitudeAndLongitude(Latitude, Longitude);

	// Find pitch/yaw/spin from the rotation
	const FQuat SphericalRotation = FRotator(-Latitude, Longitude, 0.0).Quaternion();
	const FQuat InitialRotation_SphericalRotation_LightCardOrientation = InTransform.GetRotation() * PlaneMeshRotation.Quaternion().Inverse();
	const FQuat SphericalRotation_LightCardOrientation = InOrigin.GetRotation().Inverse() * InitialRotation_SphericalRotation_LightCardOrientation;
	const FQuat LightCardOrientation = SphericalRotation.Inverse() * SphericalRotation_LightCardOrientation;
	
	const FRotator LightCardRotation = LightCardOrientation.Rotator();
	
	PositionalParams.Latitude = Latitude;
	PositionalParams.Longitude = Longitude;
	PositionalParams.DistanceFromCenter = DistanceFromCenter;
	PositionalParams.RadialOffset = InRadialOffset;
	PositionalParams.Pitch = -LightCardRotation.Pitch;
	PositionalParams.Yaw = LightCardRotation.Yaw;
	PositionalParams.Spin = LightCardRotation.Roll;
	PositionalParams.Scale = FVector2D(InTransform.GetScale3D());
	
	return PositionalParams;
}

#if WITH_EDITOR
void IDisplayClusterStageActor::UpdateEditorGizmos()
{
	if (GEditor)
	{
		if (const AActor* Actor = Cast<AActor>(this))
		{
			if (Actor->IsSelectedInEditor())
			{
				GEditor->NoteSelectionChange(/*bNotify*/ false);
			}
		}
	}
}
#endif
