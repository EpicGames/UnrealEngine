// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeature.h"
#include "Templates/SubclassOf.h"

class UActorComponent;

/** 
 * Light card Extender API
 */
class DISPLAYCLUSTERLIGHTCARDEXTENDER_API IDisplayClusterLightCardActorExtender : public IModularFeature
{
public:
	static const FName ModularFeatureName;

	virtual ~IDisplayClusterLightCardActorExtender() = default;

	/** Returns the name of the Extender */
	virtual FName GetExtenderName() const = 0;

	/** Returns the class of the additional subobject that will be created in the actor. */
	virtual TSubclassOf<UActorComponent> GetAdditionalSubobjectClass() = 0;

#if WITH_EDITOR
	/** Returns the name of the category in which details of the component are displayed */
	virtual FName GetCategory() const = 0;

	/** If true, displays properties in their respective categories */
	virtual bool ShouldShowSubcategories() const = 0;
#endif // WITH_EDITOR

};
