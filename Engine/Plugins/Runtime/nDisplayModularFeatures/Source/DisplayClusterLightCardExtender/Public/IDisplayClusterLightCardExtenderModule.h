// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

class IDisplayClusterLightCardExtenderModule : public IModuleInterface
{
public:
	static constexpr const TCHAR* ModuleName = TEXT("DisplayClusterLightCardExtender");

public:
	virtual ~IDisplayClusterLightCardExtenderModule() = default;

	/**
	* Singleton-like access to this module's interface.  This is just for convenience!
	* Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	*
	* @return Returns singleton instance, loading the module on demand if needed
	*/
	static inline IDisplayClusterLightCardExtenderModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IDisplayClusterLightCardExtenderModule>(ModuleName);
	}

	/**
	* Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	*
	* @return True if the module is loaded and ready to use
	*/
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}

#if WITH_EDITOR
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSequencerTimeChanged, TWeakPtr<class ISequencer>);
	/** Broadcast when any sequencer has its time changed, including if one is closed */
	virtual FOnSequencerTimeChanged& GetOnSequencerTimeChanged() = 0;
#endif
};