// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DisplayClusterPositionalParams.generated.h"

/**
 * Positional location and rotation parameters for use with nDisplay stage actors.
 * Note that the origin point is purposely not included as these parameters are meant to be shared between actors
 * with different origins.
 */
USTRUCT(BlueprintType)
struct FDisplayClusterPositionalParams
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Orientation)
	double DistanceFromCenter = 0.f;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Orientation, meta = (UIMin = 0, ClampMin = 0, UIMax = 360, ClampMax = 360))
	double Longitude = 0.f;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Orientation, meta = (UIMin = -90, ClampMin = -90, UIMax = 90, ClampMax = 90))
	double Latitude = 0.f;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Orientation, meta = (UIMin = -360, ClampMin = -360, UIMax = 360, ClampMax = 360))
	double Spin = 0.f;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Orientation, meta = (UIMin = -360, ClampMin = -360, UIMax = 360, ClampMax = 360))
	double Pitch = 0.f;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Orientation, meta = (UIMin = -360, ClampMin = -360, UIMax = 360, ClampMax = 360))
	double Yaw = 0.f;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Orientation)
	double RadialOffset = 0.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Orientation)
	FVector2D Scale = FVector2D(1.0f);
	
	bool operator==(const FDisplayClusterPositionalParams& Other) const
	{
		return
			FMath::IsNearlyEqual(this->DistanceFromCenter, Other.DistanceFromCenter, KINDA_SMALL_NUMBER) &&
			FMath::IsNearlyEqual(this->Longitude, Other.Longitude, KINDA_SMALL_NUMBER) &&
			FMath::IsNearlyEqual(this->Latitude,  Other.Latitude, KINDA_SMALL_NUMBER) &&
			FMath::IsNearlyEqual(this->Spin,  Other.Spin, KINDA_SMALL_NUMBER) &&
			FMath::IsNearlyEqual(this->Pitch, Other.Pitch, KINDA_SMALL_NUMBER) &&
			FMath::IsNearlyEqual(this->Yaw,   Other.Yaw, KINDA_SMALL_NUMBER) &&
			FMath::IsNearlyEqual(this->RadialOffset, Other.RadialOffset, KINDA_SMALL_NUMBER) &&
			this->Scale == Other.Scale;
	}

	bool operator!=(const FDisplayClusterPositionalParams& Other) const
	{
		return !(*this == Other);
	}
};
