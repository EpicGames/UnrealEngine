// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DisplayClusterStageActorTemplate.generated.h"

UCLASS(Abstract, PerObjectConfig, config=EditorPerProjectUserSettings)
class DISPLAYCLUSTERLIGHTCARDEXTENDER_API UDisplayClusterStageActorTemplate : public UObject
{
	GENERATED_BODY()

public:
	virtual AActor* GetTemplateActor() const { return nullptr; }
	
public:
	/** If the user has marked this a favorite template. */
	UPROPERTY(Config)
	bool bIsFavorite;
};