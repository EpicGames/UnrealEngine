// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/WeakObjectPtr.h"

class IDisplayClusterStageActor;
class AActor;

/**
 * Wrapper around a WeakObjectPtr to manage actors implementing IDisplayClusterStageActor.
 */
struct DISPLAYCLUSTERLIGHTCARDEXTENDER_API FDisplayClusterWeakStageActorPtr
{
private:
	FWeakObjectPtr ObjectPtr;

public:
	virtual ~FDisplayClusterWeakStageActorPtr() = default;
	FDisplayClusterWeakStageActorPtr() = default;
	FDisplayClusterWeakStageActorPtr(const AActor* InActorPtr);

	/** Dereference the weak pointer */
	IDisplayClusterStageActor* Get() const;
	
	AActor* AsActor() const;
	AActor* AsActorChecked() const;

protected:
	FWeakObjectPtr& GetWeakObjectPtr() { return ObjectPtr; }
	const FWeakObjectPtr& GetWeakObjectPtrConst() const { return ObjectPtr; }
	
public:
	/** Is the weak pointer valid */
	FORCEINLINE bool IsValid() const
	{
		return GetWeakObjectPtrConst().IsValid();
	}

	/** Reset the weak pointer */
	FORCEINLINE void Reset()
	{
		GetWeakObjectPtr().Reset();
	}

	/** Dereference the weak pointer */
	FORCEINLINE IDisplayClusterStageActor* operator*() const
	{
		return Get();
	}

	/** Dereference the weak pointer */
	FORCEINLINE IDisplayClusterStageActor* operator->() const
	{
		return Get();
	}

	friend bool operator==(const FDisplayClusterWeakStageActorPtr& Lhs, const FDisplayClusterWeakStageActorPtr& Rhs)
	{
		return Lhs.GetWeakObjectPtrConst() == Rhs.GetWeakObjectPtrConst();
	}

	template <typename LhsT, typename RhsT, typename = decltype((LhsT*)nullptr == (RhsT*)nullptr)>
	friend bool operator==(const FDisplayClusterWeakStageActorPtr& Lhs, const RhsT* Rhs)
	{
		return Lhs.Get() == Rhs;
	}

	template <typename LhsT, typename RhsT, typename = decltype((LhsT*)nullptr == (RhsT*)nullptr)>
	friend bool operator==(const LhsT* Lhs, const FDisplayClusterWeakStageActorPtr& Rhs)
	{
		return Lhs == Rhs.Get();
	}

	template <typename LhsT>
	friend bool operator==(const FDisplayClusterWeakStageActorPtr& Lhs, TYPE_OF_NULLPTR)
	{
		return !Lhs.IsValid();
	}

	template <typename RhsT>
	friend bool operator==(TYPE_OF_NULLPTR, const FDisplayClusterWeakStageActorPtr& Rhs)
	{
		return !Rhs.IsValid();
	}

	template <typename LhsT, typename RhsT, typename = decltype((LhsT*)nullptr != (RhsT*)nullptr)>
	friend bool operator!=(const FDisplayClusterWeakStageActorPtr& Lhs, const RhsT* Rhs)
	{
		return Lhs.Get() != Rhs;
	}

	template <typename LhsT, typename RhsT, typename = decltype((LhsT*)nullptr != (RhsT*)nullptr)>
	friend bool operator!=(const LhsT* Lhs, const FDisplayClusterWeakStageActorPtr& Rhs)
	{
		return Lhs != Rhs.Get();
	}

	template <typename LhsT>
	friend bool operator!=(const FDisplayClusterWeakStageActorPtr& Lhs, TYPE_OF_NULLPTR)
	{
		return Lhs.IsValid();
	}

	template <typename RhsT>
	friend bool operator!=(TYPE_OF_NULLPTR, const FDisplayClusterWeakStageActorPtr& Rhs)
	{
		return Rhs.IsValid();
	}
};
