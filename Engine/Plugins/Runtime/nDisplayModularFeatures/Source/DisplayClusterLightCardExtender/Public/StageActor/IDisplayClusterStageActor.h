// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisplayClusterPositionalParams.h"

#include "UObject/Interface.h"

#include "IDisplayClusterStageActor.generated.h"

UINTERFACE(MinimalAPI, meta = (CannotImplementInterfaceInBlueprint))
class UDisplayClusterStageActor : public UInterface
{
	GENERATED_BODY()
};

class DISPLAYCLUSTERLIGHTCARDEXTENDER_API IDisplayClusterStageActor
{
	GENERATED_BODY()

public:
	/** A pair made up of a container pointer and a property within that container */
	typedef TPair<void*, FProperty*> FPropertyPair;

	/** Array type used to return positional properties for a stage actor */
	typedef TArray<FPropertyPair, TInlineAllocator<16>> FPositionalPropertyArray;

public:
	/** The rotation used to orient the plane mesh used for the light card so that its normal points radially inwards */
	static const FRotator PlaneMeshRotation;

	virtual void SetLongitude(double InValue) = 0;
	virtual double GetLongitude() const = 0;

	virtual void SetLatitude(double InValue) = 0;
	virtual double GetLatitude() const = 0;

	virtual void SetDistanceFromCenter(double InValue) = 0;
	virtual double GetDistanceFromCenter() const = 0;

	virtual void SetSpin(double InValue) = 0;
	virtual double GetSpin() const = 0;

	virtual void SetPitch(double InValue) = 0;
	virtual double GetPitch() const = 0;

	virtual void SetYaw(double InValue) = 0;
	virtual double GetYaw() const = 0;

	virtual void SetRadialOffset(double InValue) = 0;
	virtual double GetRadialOffset() const = 0;

	virtual void SetOrigin(const FTransform& InOrigin) = 0;
	virtual FTransform GetOrigin() const = 0;

	virtual void SetScale(const FVector2D& InScale) = 0;
	virtual FVector2D GetScale() const = 0;

	virtual void SetAlwaysFlushToWall(bool bInAlwaysFlushToWall) { }
	virtual bool IsAlwaysFlushToWall() const { return true; }


	/** Return property names defined for this stage actor for use in property notifies */
	virtual const TSet<FName>& GetPositionalPropertyNames() const;

	/** Return the property name for an owning struct containing the positional params properties, may be NAME_None if a struct isn't used */
	virtual FName GetPositionalPropertiesMemberName() const;

	/** Get an array of positional properties defined for this stage actor for use in property notifies. */
	virtual void GetPositionalProperties(FPositionalPropertyArray& OutPropertyPairs) const = 0;

	/** Return the bounding box for the stage actor. The default implementation returns the components bounding box */
	virtual FBox GetBoxBounds(bool bLocalSpace = false) const;

	/** Whether this actor supports movement in UV space */
	virtual bool IsUVActor() const { return false; }
	virtual void SetUVCoordinates(const FVector2D& InUVCoordinates) {}
	virtual FVector2D GetUVCoordinates() const { return FVector2D(); }

	/** If this stage actor is a copy of a level instance actor */
	virtual bool IsProxy() const;

	/** Update positional properties of the stage actor. The default implementation updates the actor's transform */
	virtual void UpdateStageActorTransform();

	/** Return the adjusted transform of the stage actor. The default implementation uses the actor's location as a base */
	virtual FTransform GetStageActorTransform(bool bRemoveOrigin = false) const;

	/** Sets positional parameters */
	virtual void SetPositionalParams(const FDisplayClusterPositionalParams& InParams);
	
	/** Retrieves positional parameters */
	virtual FDisplayClusterPositionalParams GetPositionalParams() const;
	
	/** Update positional params from current actor transform */
	virtual void UpdatePositionalParamsFromTransform();
	
	/** Clamp the given latitude and longitude */
	static void ClampLatitudeAndLongitude(double& InOutLatitude, double& InOutLongitude);

	/** Convert latitude/longitude/yaw/pitch/roll to a transform that could be applied to an actor  */
	static FTransform PositionalParamsToActorTransform(const FDisplayClusterPositionalParams& InParams, const FTransform& InOrigin);

	/** Convert an actor transform to positional params */
	static FDisplayClusterPositionalParams TransformToPositionalParams(const FTransform& InTransform, const FTransform& InOrigin, double InRadialOffset);

#if WITH_EDITOR
	/** Update global selection gizmos */
	void UpdateEditorGizmos();
#endif
};
