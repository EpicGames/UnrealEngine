// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class DisplayClusterModularFeaturesEditor : ModuleRules
{
	public DisplayClusterModularFeaturesEditor(ReadOnlyTargetRules ROTargetRules) : base(ROTargetRules)
	{
		ShortName = "NDCModFeaturesEd";

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine"
			});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"UnrealEd"
			});
	}
}
