// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeature.h"

/**
 * Container to carry the info about media object's owner
 */
struct FMediaObjectOwnerInfo
{
	/**
	 * Type of the media object's owner
	 */
	enum class EMediaObjectOwnerType : uint8
	{
		ICVFXCamera = 0,
		Viewport,
		Backbuffer
	};


	/** Owner name (ICVFX camera component name, viewport or node name) */
	FString OwnerName;

	/** Owner type (ICVFX camera component name, viewport or node name) */
	EMediaObjectOwnerType OwnerType;

	/** Optional unique index of the cluster node holding the owner object */
	TOptional<uint8> ClusterNodeUniqueIdx;

	/**
	 * Unique index of the owner
	 *   Camera     - within a config
	 *   Viewport   - within a cluster node
	 *   Backbuffer - within a config
	 */
	uint8 OwnerUniqueIdx = 0;
};

/**
 * Media stream propagation types
 */
enum class EMediaStreamPropagationType : uint8
{
	None = 0,
	LocalUnicast      = 1 << 0, // Single sender and single receiver within the same host
	LocalMulticast    = 1 << 1, // Single sender and multiple receivers within the same host
	Unicast           = 1 << 2, // Single sender and single receiver on different hosts
	Multicast         = 1 << 3  // Single sender and multiple receivers on different hosts
};
ENUM_CLASS_FLAGS(EMediaStreamPropagationType);


/**
 * Base class for nDisplay media initializer implementations.
 */
class DISPLAYCLUSTERMODULARFEATURESEDITOR_API IDisplayClusterModularFeatureMediaInitializer
	: public IModularFeature
{
public:
	/** Public feature name */
	static const FName ModularFeatureName;

public:

	virtual ~IDisplayClusterModularFeatureMediaInitializer() = default;

public:

	/**
	 * Checks if media object is supported by the initializer
	 *
	 * @param MediaObject - UMediaSource or UMediaOutput instance
	 * @return true if media object supported
	 */
	virtual bool IsMediaObjectSupported(const UObject* MediaObject) = 0;

	/**
	 * Checks if media source and output are compatible and can be paired
	 *
	 * @param MediaSource - UMediaSource instance
	 * @param MediaOutput - UMediaOutput instance
	 * @return true if media objects are compatible with each other
	 */
	virtual bool AreMediaObjectsCompatible(const UObject* MediaSource, const UObject* MediaOutput) = 0;

	/**
	 * Provides stream supported media propagation types (local/global, unicast/multicast, etc.)
	 *
	 * @param MediaSource - UMediaSource instance
	 * @param MediaOutput - UMediaOutput instance
	 * @param OutPropagationTypes - [out] supported types mask
	 * @return false if media objects are invalid, incompatible or not supported
	 */
	virtual bool GetSupportedMediaPropagationTypes(const UObject* MediaSource, const UObject* MediaOutput, EMediaStreamPropagationType& OutPropagationTypes) = 0;

	/**
	 * Performs initialization of a media object for tiled input/output
	 *
	 * @param MediaObject - UMediaSource or UMediaOutput instance
	 * @param OwnerInfo   - Additional information about the object holding the media object
	 * @param TilePos     - Tile XY-position
	 */
	virtual void InitializeMediaObjectForTile(UObject* MediaObject, const FMediaObjectOwnerInfo& OnwerInfo, const FIntPoint& TilePos) = 0;

	/**
	 * Performs initialization of a media object for full frame input/output
	 *
	 * @param MediaObject - UMediaSource or UMediaOutput instance
	 * @param OwnerInfo   - Additional information about the object holding the media object
	 */
	virtual void InitializeMediaObjectForFullFrame(UObject* MediaObject, const FMediaObjectOwnerInfo& OnwerInfo) = 0;
};
