{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "nDisplay Modular Features",
	"Description": "Modular Features for nDisplay",
	"Category": "Misc",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Hidden": true,
	"Modules": [
		{
			"Name": "DisplayClusterLightCardExtender",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DisplayClusterModularFeaturesEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}
